// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CmdLink : ModuleRules
{
    public CmdLink(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Core",
				"Projects",
            }
        );
        
        PrivateIncludePathModuleNames.AddRange(
	        new string[] {
		        "Launch",
		        "TargetPlatform",
	        });
    }
}