// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

[SupportedPlatforms("Win64")]
public class CmdLinkTarget : TargetRules
{
	public CmdLinkTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "CmdLink";
		bLegalToDistributeBinary = true;

		// Turn off various third party features we don't need

		// Lean and mean
		bBuildDeveloperTools = false;

		// CmdLink isn't localized, so doesn't need ICU
		bCompileICU = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bBuildWithEditorOnlyData = false;
		bCompileCEF3 = false;

		// CmdLink is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
