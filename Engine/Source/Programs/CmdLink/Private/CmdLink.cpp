// Copyright Epic Games, Inc. All Rights Reserved.

#include <iostream>

#include "Core.h"
#include "Windows/WindowsPlatformNamedPipe.h"

#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(CmdLink, "CmdLink")

enum EErrorMessage
{
	SUCCESS = 0,
	CONNECT_FAILED,
	WRITE_FAILED,
	READ_FAILED
};

EErrorMessage PrintError(EErrorMessage Error)
{
	switch(Error)
	{
	case CONNECT_FAILED:
		printf("Failed to connect to UE\n");
		break;
	case WRITE_FAILED:
		printf("Failed to send command to UE\n");
		break;
	case READ_FAILED:
		printf("Failed to read response from UE\n");
		break;
	}
	return Error;
}

#define TRY(code, error) if(!(code)) return PrintError(error)

int32 main(int32 ArgC, ANSICHAR* ArgV[])
{
	FString CLIPipeKey;
	if (ArgC >= 2 && FParse::Value(StringCast<TCHAR>(ArgV[1]).Get(),TEXT("--LinkKey="),CLIPipeKey))
	{
		// don't send this argument to the server. it's meant for Client only
		--ArgC;
		++ArgV;
	}
	
	// The CLIServer expects messages from the client in the following format:
	// 1. int32: ArgC
	// repeat lines 2 and 3 ArgC times
	// 2. int32: ArgLen
	// 3. char[Arglen]: Arg (includes null terminating character)

	// the CmdLink client expects a response from the server in the following format:
	// 1. int32: responseLen
	// 2. char[responseLen] response (includes null terminating character)
	
	const FString PipeName = CLIPipeKey.IsEmpty() ? TEXT("UnrealEngine-CLI") : TEXT("UnrealEngine-CLI-") + CLIPipeKey;
	FPlatformNamedPipe NamedPipe;
	TRY(NamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s"), *PipeName), false, false), CONNECT_FAILED);

	// Send ArgC
	TRY(NamedPipe.WriteInt32(ArgC), WRITE_FAILED);

	// send file path of caller
	{
		FString LaunchDir = FPlatformProcess::GetCurrentWorkingDirectory();
		FPaths::NormalizeDirectoryName(LaunchDir);
		const TStringConversion AnsiStr = StringCast<ANSICHAR>(*LaunchDir);
		const int32 Size = AnsiStr.Length() + 1;
		TRY(NamedPipe.WriteInt32(Size), WRITE_FAILED);
		TRY(NamedPipe.WriteBytes(Size, AnsiStr.Get()), WRITE_FAILED);
	}

	// Send ArgV (skipping filename)
	for(int i = 1; i < ArgC; ++i)
	{
		const int32 Size = TCString<ANSICHAR>::Strlen(ArgV[i]) + 1;
		TRY(NamedPipe.WriteInt32(Size), WRITE_FAILED);
		TRY(NamedPipe.WriteBytes(Size, ArgV[i]), WRITE_FAILED);
	}

	// Receive Response string
	int32 Size;
	TRY(NamedPipe.ReadInt32(Size), READ_FAILED);
	ANSICHAR* Buffer = new ANSICHAR[Size];
	TRY(NamedPipe.ReadBytes(Size, Buffer), READ_FAILED);

	// Print Response
	check(Buffer[Size - 1] == '\0');
	printf(Buffer);

	delete [] Buffer;
	NamedPipe.Destroy();
	return SUCCESS;
}
